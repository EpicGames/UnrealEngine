// Copyright Epic Games, Inc. All Rights Reserved.

#include "SvgDistanceFieldGenerate.h"

#if WITH_EDITOR

// Include Windows.h before Skia so that its unnecessary defines are cleaned up and Skia's include will be ignored
#if PLATFORM_WINDOWS
#include "Windows/WindowsHWrapper.h"
#endif

#if PLATFORM_MAC
#include "Mac/MacSystemIncludes.h"
#endif

#include <stack>
#define MSDFGEN_NO_FREETYPE
#define MSDFGEN_PARENT_NAMESPACE
#define SKIA_SIMPLIFY_NAMESPACE
#define MSDFGEN_ENABLE_SVG
#define MSDFGEN_USE_SKIA
#define MSDFGEN_USE_DROPXML
#include "ThirdParty/skia/skia-simplify.cpp"
#include "ThirdParty/msdfgen/dropXML.hpp"
#define fopen(...) nullptr // Not used but prevent deprecation warnings
#include "ThirdParty/msdfgen/msdfgen.cpp"

bool SvgDistanceFieldGenerate(TArrayView64<const char> InSvgData, const FSvgDistanceFieldConfiguration& InConfiguration, FDistanceFieldImage& OutDistanceFieldImage)
{
	float OuterUnitSpread = 0.f;
	float InnerUnitSpread = 0.f;
	float OuterPxSpread = 0.f;
	float InnerPxSpread = 0.f;

	switch (InConfiguration.DistanceSpreadUnits)
	{
		case ESvgDistanceFieldUnits::SvgUnits:
			OuterUnitSpread = .5f*InConfiguration.BaseDistanceSpread + InConfiguration.ExtraOuterDistanceSpread;
			InnerUnitSpread = .5f*InConfiguration.BaseDistanceSpread + InConfiguration.ExtraInnerDistanceSpread;
			break;
		case ESvgDistanceFieldUnits::OutputPixels:
			OuterPxSpread = .5f*InConfiguration.BaseDistanceSpread + InConfiguration.ExtraOuterDistanceSpread;
			InnerPxSpread = .5f*InConfiguration.BaseDistanceSpread + InConfiguration.ExtraInnerDistanceSpread;
			break;
		case ESvgDistanceFieldUnits::ProportionalToMaxDimension:
			OuterPxSpread = InnerPxSpread = (float) FMath::Max(InConfiguration.OutputWidth, InConfiguration.OutputHeight);
			OuterPxSpread *= .5f*InConfiguration.BaseDistanceSpread + InConfiguration.ExtraOuterDistanceSpread;
			InnerPxSpread *= .5f*InConfiguration.BaseDistanceSpread + InConfiguration.ExtraInnerDistanceSpread;
		default:
			checkNoEntry();
			return false;
	}

	const float MiterLimit = InConfiguration.DistanceFieldType != ESvgDistanceFieldType::Simple ? InConfiguration.MiterLimit : 0.f;

	if (!(
		OuterUnitSpread + InnerUnitSpread >= 0.f &&
		OuterPxSpread + InnerPxSpread >= 0.f &&
		(OuterUnitSpread + InnerUnitSpread > 0.f || OuterPxSpread + InnerPxSpread > 0.f) &&
		InConfiguration.OutputWidth > 0 &&
		InConfiguration.OutputHeight > 0 &&
		InConfiguration.MiterLimit >= 0.f //-V1051
	)) {
		return false;
	}

	msdfgen::Shape SvgShape;
	msdfgen::Shape::Bounds SvgViewBox = { };
	const int ParseResult = msdfgen::parseSvgShape(SvgShape, SvgViewBox, InSvgData.GetData(), InSvgData.Num());
	if (!(ParseResult&msdfgen::SVG_IMPORT_SUCCESS_FLAG))
	{
		return false;
	}
	SvgShape.inverseYAxis = !SvgShape.inverseYAxis;

	msdfgen::Shape::Bounds Bounds = { };
	if (InConfiguration.ScaleMode == ESvgDistanceFieldScaleMode::FitBoundingBox || InConfiguration.PlacementMode == ESvgDistanceFieldPlacementMode::CenterBoundingBox)
	{
		Bounds = SvgShape.getBounds(OuterUnitSpread, OuterUnitSpread > 0.f ? MiterLimit : 0.f);
	}

	float Scale = 1.f;
	msdfgen::Vector2 Translate;
	msdfgen::MSDFGeneratorConfig MsdfgenConfig;
	MsdfgenConfig.overlapSupport = false;
	switch (InConfiguration.ScaleMode)
	{
		case ESvgDistanceFieldScaleMode::ExplicitScale:
			Scale = InConfiguration.Scale;
			break;
		case ESvgDistanceFieldScaleMode::FitCanvas:
			Scale = FMath::Min(
				(float) InConfiguration.OutputWidth / (float) (SvgViewBox.r - SvgViewBox.l),
				(float) InConfiguration.OutputHeight / (float) (SvgViewBox.t - SvgViewBox.b)
			);
			break;
		case ESvgDistanceFieldScaleMode::FitPaddedCanvas:
			Scale = FMath::Min(
				((float) InConfiguration.OutputWidth - 2.f*OuterPxSpread) / ((float) (SvgViewBox.r-SvgViewBox.l) + 2.f*OuterUnitSpread),
				((float) InConfiguration.OutputHeight - 2.f*OuterPxSpread) / ((float) (SvgViewBox.t-SvgViewBox.b) + 2.f*OuterUnitSpread)
			);
			break;
		case ESvgDistanceFieldScaleMode::FitBoundingBox:
			Scale = FMath::Min(
				((float) InConfiguration.OutputWidth - 2.f*OuterPxSpread) / (float) (Bounds.r - Bounds.l),
				((float) InConfiguration.OutputHeight - 2.f*OuterPxSpread) / (float) (Bounds.t - Bounds.b)
			);
			break;
		default:
			checkNoEntry();
	}

	if (Scale <= 0.f || !FMath::IsFinite(Scale))
	{
		return false;
	}

	const float TotalOuterPxSpread = OuterPxSpread + Scale*OuterUnitSpread;
	const float TotalInnerPxSpread = InnerPxSpread + Scale*InnerUnitSpread;
	const float TotalOuterUnitSpread = OuterUnitSpread + OuterPxSpread/Scale;
	const float TotalInnerUnitSpread = InnerUnitSpread + InnerPxSpread/Scale;

	switch (InConfiguration.PlacementMode)
	{
		case ESvgDistanceFieldPlacementMode::DoNotTranslate:
			break;
		case ESvgDistanceFieldPlacementMode::PadWithOuterSpread:
			Translate = msdfgen::Vector2(TotalOuterUnitSpread, TotalOuterUnitSpread);
			break;
		case ESvgDistanceFieldPlacementMode::CenterCanvas:
			Translate = .5*(msdfgen::Vector2(InConfiguration.OutputWidth, InConfiguration.OutputHeight)/Scale - msdfgen::Vector2(SvgViewBox.r-SvgViewBox.l, SvgViewBox.t-SvgViewBox.b));
			break;
		case ESvgDistanceFieldPlacementMode::CenterBoundingBox:
			Translate = .5*(msdfgen::Vector2(InConfiguration.OutputWidth, InConfiguration.OutputHeight)/Scale - msdfgen::Vector2(Bounds.r-Bounds.l, Bounds.t-Bounds.b)) - msdfgen::Vector2(Bounds.l, Bounds.b);
			break;
	}
	const msdfgen::SDFTransformation Transformation(msdfgen::Projection(Scale, Translate), msdfgen::Range(-TotalOuterUnitSpread, TotalInnerUnitSpread));

	const bool bMsdf = InConfiguration.DistanceFieldType == ESvgDistanceFieldType::MultiChannelAndSimple;
	OutDistanceFieldImage.RawPixelData.SetNumUninitialized((uint64) (bMsdf ? 4 : 1)*InConfiguration.OutputWidth*InConfiguration.OutputHeight);
	OutDistanceFieldImage.PixelFormat = bMsdf ? PF_B8G8R8A8 : PF_G8;
	OutDistanceFieldImage.Format = bMsdf ? TSF_BGRA8 : TSF_G8;
	OutDistanceFieldImage.CompressionSettings = bMsdf ? TC_VectorDisplacementmap : TC_Displacementmap;
	OutDistanceFieldImage.SizeX = InConfiguration.OutputWidth;
	OutDistanceFieldImage.SizeY = InConfiguration.OutputHeight;

	// Temporarily use output data buffer as error correction buffer
	MsdfgenConfig.errorCorrection.buffer = reinterpret_cast<msdfgen::byte*>(OutDistanceFieldImage.RawPixelData.GetData());
	TArray64<float> FloatBitmapData;
	FloatBitmapData.SetNumUninitialized(OutDistanceFieldImage.RawPixelData.Num());

	switch (InConfiguration.DistanceFieldType)
	{
		case ESvgDistanceFieldType::Simple:
		{
			msdfgen::BitmapRef<float, 1> FloatBitmap(FloatBitmapData.GetData(), InConfiguration.OutputWidth, InConfiguration.OutputHeight);
			msdfgen::generateSDF(FloatBitmap, SvgShape, Transformation, MsdfgenConfig);
			break;
		}
		case ESvgDistanceFieldType::Perpendicular:
		{
			msdfgen::BitmapRef<float, 1> FloatBitmap(FloatBitmapData.GetData(), InConfiguration.OutputWidth, InConfiguration.OutputHeight);
			msdfgen::generatePSDF(FloatBitmap, SvgShape, Transformation, MsdfgenConfig);
			break;
		}
		case ESvgDistanceFieldType::MultiChannelAndSimple:
		{
			msdfgen::BitmapRef<float, 4> FloatBitmap(FloatBitmapData.GetData(), InConfiguration.OutputWidth, InConfiguration.OutputHeight);
			msdfgen::edgeColoringSimple(SvgShape, 3);
			msdfgen::generateMTSDF(FloatBitmap, SvgShape, Transformation, MsdfgenConfig);
			break;
		}
		default:
			checkNoEntry();
			return false;
	}

	const float* Src = FloatBitmapData.GetData();
	for (uint8* Dst = OutDistanceFieldImage.RawPixelData.GetData(), * End = Dst+OutDistanceFieldImage.RawPixelData.Num(); Dst < End; ++Dst, ++Src)
	{
		*Dst = msdfgen::pixelFloatToByte(*Src);
	}
	return true;
}

#else

bool SvgDistanceFieldGenerate(TArrayView64<const char> InSvgData, const FSvgDistanceFieldConfiguration& InConfiguration, FDistanceFieldImage &OutDistanceFieldImage)
{
	return false;
}

#endif
