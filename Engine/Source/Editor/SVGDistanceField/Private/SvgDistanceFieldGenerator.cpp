// Copyright Epic Games, Inc. All Rights Reserved.

#include "SvgDistanceFieldGenerator.h"

#include "Misc/FileHelper.h"
#include "DistanceFieldImage.h"
#include "SvgDistanceFieldGenerate.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SvgDistanceFieldGenerator)

#if WITH_EDITORONLY_DATA
UTexture2D* USvgDistanceFieldGenerator::GenerateTextureFromSvgFile(const FString& SvgFilePath, const FSvgDistanceFieldConfiguration& Configuration)
{
#if WITH_EDITOR
	TArray64<uint8> SvgData;
	if (FFileHelper::LoadFileToArray(SvgData, *SvgFilePath))
	{
		FDistanceFieldImage Image;
		if (SvgDistanceFieldGenerate(TArrayView64<const char>(reinterpret_cast<const char*>(SvgData.GetData()), SvgData.Num()), Configuration, Image))
		{
			UTexture2D* Texture = UTexture2D::CreateTransient(Image.SizeX, Image.SizeY, Image.PixelFormat, NAME_None, Image.RawPixelData);
			Texture->PreEditChange(nullptr);
			Texture->SRGB = Image.bSRGB;
			Texture->CompressionSettings = Image.CompressionSettings;
			Texture->MipGenSettings = Image.MipGenSettings;
			Texture->PostEditChange();
			return Texture;
		}
	}
#endif
	return nullptr;
}
#endif
