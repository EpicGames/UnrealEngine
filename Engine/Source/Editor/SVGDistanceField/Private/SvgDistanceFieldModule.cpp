// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"

class FSvgDistanceFieldModule : public IModuleInterface
{
public:
	// IModuleInterface interface
	virtual void StartupModule() override { }
	virtual void ShutdownModule() override { }

};

IMPLEMENT_MODULE(FSvgDistanceFieldModule, SVGDistanceField);
