// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "PixelFormat.h"
#include "Engine/TextureDefines.h"

struct FDistanceFieldImage
{
	TArray64<uint8> RawPixelData;
	static constexpr ETextureSourceCompressionFormat RawPixelDataCompressionFormat = TSCF_None;
	EPixelFormat PixelFormat = PF_Unknown;
	ETextureSourceFormat Format = TSF_Invalid;
	TextureCompressionSettings CompressionSettings = TC_Default;
	int32 SizeX = 0;
	int32 SizeY = 0;
	static constexpr bool bSRGB = false;
	static constexpr TextureMipGenSettings MipGenSettings = TMGS_NoMipmaps;
};
