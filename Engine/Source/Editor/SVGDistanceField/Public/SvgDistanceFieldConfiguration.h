// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "SvgDistanceFieldConfiguration.generated.h"

/** The possible types of distance fields that can be generated */
UENUM(BlueprintType)
enum class ESvgDistanceFieldType : uint8
{
	/** Simple single-channel Euclidean signed distance field */
	Simple,
	/** Single-channel perpendicular-distance field - distance from corners is represented so that extrusions are not rounded. */
	Perpendicular,
	/** Multi-channel signed distance field with simple Euclidean distance field in the alpha channel */
	MultiChannelAndSimple
};

/** The units in which certain values are expressed */
UENUM(BlueprintType)
enum class ESvgDistanceFieldUnits : uint8
{
	/** The "user units" sometimes also referred to as pixels in the input SVG file */
	SvgUnits,
	/** Pixels of the output image */
	OutputPixels,
	/** Proportion of the larger dimension of the output image */
	ProportionalToMaxDimension
};

/** The strategy of scaling the SVG coordinate system units to output pixels */
UENUM(BlueprintType)
enum class ESvgDistanceFieldScaleMode : uint8
{
	/** The SVG coordinates are to be multiplied by the Scale value. */
	ExplicitScale,
	/** Scale is selected so that the entire SVG canvas tightly fits the output image. */
	FitCanvas,
	/** Scale is selected so that the SVG canvas padded by the outer distance spread on each side tightly fits the output image. */
	FitPaddedCanvas,
	/** Scale is selected so that all areas with a signed distance above the minimum representable value tightly fit the output image. */
	FitBoundingBox
};

/** The strategy for positioning the SVG geometry within the output image */
UENUM(BlueprintType)
enum class ESvgDistanceFieldPlacementMode : uint8
{
	/** Makes sure not to shift the coordinate system at all. */
	DoNotTranslate,
	/** Shifts the coordinate system predictably (by outer distance spread in both dimensions) to account for the outer distance gradient around the geometry. */
	PadWithOuterSpread,
	/** Pins the center of the SVG canvas to the center of the output image. */
	CenterCanvas,
	/** Centers the actual geometry in such a way that left/right and top/bottom margins are the same. */
	CenterBoundingBox
};

USTRUCT(BlueprintType)
struct FSvgDistanceFieldConfiguration
{
	GENERATED_USTRUCT_BODY()

	/** Sets which type of signed distance field should be generated. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=SvgDistanceField)
	ESvgDistanceFieldType DistanceFieldType = ESvgDistanceFieldType::Simple;

	/** The base portion of the representable distance spread that will be divided equally between the inside and outside. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=SvgDistanceField)
	float BaseDistanceSpread = 4.f;

	/** The additional asymmetrical outer portion of the representable distance spread. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=SvgDistanceField)
	float ExtraOuterDistanceSpread = 0.f;

	/** The additional asymmetrical inner portion of the representable distance spread. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=SvgDistanceField)
	float ExtraInnerDistanceSpread = 0.f;

	/** Specifies the units in which all distance spread values are expressed. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=SvgDistanceField)
	ESvgDistanceFieldUnits DistanceSpreadUnits = ESvgDistanceFieldUnits::OutputPixels;

	/** The width of the output image in pixels */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=SvgDistanceField, meta=(ClampMin=1))
	int32 OutputWidth = 256;

	/** The height of the output image in pixels */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=SvgDistanceField, meta=(ClampMin=1))
	int32 OutputHeight = 256;

	/** Specifies how the SVG coordinates are scaled when converting to output pixel coordinates. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=SvgDistanceField)
	ESvgDistanceFieldScaleMode ScaleMode = ESvgDistanceFieldScaleMode::ExplicitScale;

	/** The scaling factor from SVG coordinates to pixel coordinates if ScaleMode is set to ExplicitScale, otherwise ignored */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=SvgDistanceField, meta=(ClampMin=0, EditCondition="ScaleMode == ESvgDistanceFieldScaleMode::ExplicitScale"))
	float Scale = 1.f;

	/** Specifies how the SVG coordinates are translated when converting to output pixel coordinates. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=SvgDistanceField)
	ESvgDistanceFieldPlacementMode PlacementMode = ESvgDistanceFieldPlacementMode::DoNotTranslate;

	/** If DistanceFieldType is set to Perpendicular or MultiChannelAndSimple, specifies the miter limit when computing the bounding box. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=SvgDistanceField, meta=(ClampMin=0, EditCondition="DistanceFieldType != ESvgDistanceFieldType::Simple"))
	float MiterLimit = 0.f;

};
