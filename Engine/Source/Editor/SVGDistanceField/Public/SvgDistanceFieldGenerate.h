// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "DistanceFieldImage.h"
#include "SvgDistanceFieldConfiguration.h"

bool SVGDISTANCEFIELD_API SvgDistanceFieldGenerate(TArrayView64<const char> InSvgData, const FSvgDistanceFieldConfiguration& InConfiguration, FDistanceFieldImage& OutDistanceFieldImage);
