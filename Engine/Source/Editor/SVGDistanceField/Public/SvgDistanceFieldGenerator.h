// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Engine/Texture2D.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "SvgDistanceFieldConfiguration.h"

#include "SvgDistanceFieldGenerator.generated.h"

#define UE_API SVGDISTANCEFIELD_API

UCLASS(MinimalAPI)
class USvgDistanceFieldGenerator : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:

#if WITH_EDITORONLY_DATA
	UFUNCTION(BlueprintCallable, Category=SvgDistanceField)
	static UE_API UTexture2D* GenerateTextureFromSvgFile(const FString& SvgFilePath, const FSvgDistanceFieldConfiguration& Configuration);
#endif

};

#undef UE_API
