// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

[SupportedTargetTypes(TargetType.Editor, TargetType.Program)]
public class SVGDistanceField : ModuleRules
{
	public SVGDistanceField(ReadOnlyTargetRules Target) : base(Target)
	{
		bRequiresPlatformSDK = true;
		
		PublicDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CoreUObject",
				"Engine"
			}
		);
	}
}
